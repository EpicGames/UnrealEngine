// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using Microsoft.Extensions.Logging;
using System;
using System.Linq;
using UnrealBuildBase;
using UnrealBuildTool;

public class BuildSettings : ModuleRules
{
	public BuildSettings(ReadOnlyTargetRules Target) : base(Target)
	{
		CppCompileWarningSettings.DeterministicWarningLevel = WarningLevel.Off; // This module intentionally uses __DATE__ and __TIME__ macros
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

		PrivateIncludePathModuleNames.Add("Core");

		bRequiresImplementModule = false;

		if (Target.bBuildEditor)
		{
			bDisableAutoRTFMInstrumentation = true;
		}

		PrivateDefinitions.Add($"ENGINE_VERSION_MAJOR={Target.Version.MajorVersion}");
		PrivateDefinitions.Add($"ENGINE_VERSION_MINOR={Target.Version.MinorVersion}");
		PrivateDefinitions.Add($"ENGINE_VERSION_HOTFIX={Target.Version.PatchVersion}");
		PrivateDefinitions.Add($"ENGINE_VERSION_STRING=\"{Target.Version.MajorVersion}.{Target.Version.MinorVersion}.{Target.Version.PatchVersion}-{Target.BuildVersion}\"");
		PrivateDefinitions.Add($"ENGINE_IS_LICENSEE_VERSION={(Target.Version.IsLicenseeVersion ? "true" : "false")}");
		PrivateDefinitions.Add($"ENGINE_IS_PROMOTED_BUILD={(Target.Version.IsPromotedBuild ? "true" : "false")}");

		if (!Target.GlobalDefinitions.Any(x => x.Contains("CURRENT_CHANGELIST", StringComparison.Ordinal)))
		{
			PrivateDefinitions.Add($"CURRENT_CHANGELIST={Target.Version.Changelist}");
		}

		PrivateDefinitions.Add($"COMPATIBLE_CHANGELIST={Target.Version.EffectiveCompatibleChangelist}");
		PrivateDefinitions.Add($"BRANCH_NAME=\"{Target.Version.BranchName}\"");
		PrivateDefinitions.Add($"BUILD_VERSION=\"{Target.BuildVersion}\"");
		PrivateDefinitions.Add($"BUILD_SOURCE_URL=\"{Target.Version.BuildURL}\"");

		PrivateDefinitions.Add($"BUILD_USER=\"{(Target.bEnablePrivateBuildInformation ? Environment.UserName : String.Empty)}\"");
		PrivateDefinitions.Add($"BUILD_USERDOMAINNAME=\"{(Target.bEnablePrivateBuildInformation ? Environment.UserDomainName : String.Empty)}\"");
		PrivateDefinitions.Add($"BUILD_MACHINENAME=\"{(Target.bEnablePrivateBuildInformation ? Unreal.MachineName : String.Empty)}\"");

		PrivateDefinitions.Add("SUPPRESS_PER_MODULE_INLINE_FILE"); // This module does not use core's standard operator new/delete overloads

		if (Target.bWithLiveCoding && Target.LinkType == TargetLinkType.Monolithic)
		{
			PrivateDefinitions.Add($"UE_LIVE_CODING_ENGINE_DIR=\"{Unreal.EngineDirectory.FullName.Replace("\\", "\\\\")}\"");
			if (Target.ProjectFile != null)
			{
				PrivateDefinitions.Add($"UE_LIVE_CODING_PROJECT=\"{Target.ProjectFile.FullName.Replace("\\", "\\\\")}\"");
			}
		}

		PrivateDefinitions.Add($"UE_PERSISTENT_ALLOCATOR_RESERVE_SIZE={GetPersistentAllocatorReserveSize()}ULL");

		if (!FilesToGenerate.ContainsKey("RHIStaticShaderPlatformNames.gen.h"))
		{
			FilesToGenerate.TryAdd("RHIStaticShaderPlatformNames.gen.h", [
				"#pragma once",
				"#include \"HAL/Platform.h\"",
				"struct FStaticNameMapEntry { const TCHAR* Name; const TCHAR* Platform; const int Enum; };",
				"extern BUILDSETTINGS_API FStaticNameMapEntry GStaticShaderNames[];"
			]);
			FilesToGenerate.Add("RHIStaticShaderPlatformNames.gen.cpp", [
				"#include \"RHIStaticShaderPlatformNames.gen.h\"",
				"BUILDSETTINGS_API FStaticNameMapEntry GStaticShaderNames[] = ",
				"{",
				"	{ nullptr, nullptr, 0 },",
				"};"
			]);
		}
	}

	private ulong GetPersistentAllocatorReserveSize()
	{
		// We have to separate persistent allocator reserve into Editor and Engine as some platforms, like iOS have very limited VM by default, so we can't reserve a lot of vm just in case
		ConfigHierarchyType ConfigHierarchy = Target.Type == TargetType.Editor ? ConfigHierarchyType.Editor : ConfigHierarchyType.Engine;
		ConfigHierarchy Ini = ConfigCache.ReadHierarchy(ConfigHierarchy, Target.BuildEnvironment == TargetBuildEnvironment.Unique ? DirectoryReference.FromFile(Target.ProjectFile) : null, Target.Platform);
		if (Ini.GetInt32("/Script/Engine.GarbageCollectionSettings", "gc.SizeOfPermanentObjectPool", out int _))
		{
			Target.Logger.LogWarning("/Script/Engine.GarbageCollectionSettings, gc.SizeOfPermanentObjectPool ini for Project {Project} setting was deprecated in a favor of MemoryPools, PersistentAllocatorReserveSizeMB", Target.ProjectFile);
		}
		Ini.GetInt32("MemoryPools", "PersistentAllocatorReserveSizeMB", out int PersistentAllocatorReserveSizeMB);
		return (ulong)PersistentAllocatorReserveSizeMB * 1024 * 1024;
	}
}
