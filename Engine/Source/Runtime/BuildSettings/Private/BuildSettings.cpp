// Copyright Epic Games, Inc. All Rights Reserved.

#include "BuildSettings.h"
#include "Containers/StringView.h"
#include "Misc/CString.h"

namespace BuildSettings
{
	bool IsLicenseeVersion()
	{
		return ENGINE_IS_LICENSEE_VERSION;
	}

	int GetEngineVersionMajor()
	{
		return ENGINE_VERSION_MAJOR;
	}

	int GetEngineVersionMinor()
	{
		return ENGINE_VERSION_MINOR;
	}

	int GetEngineVersionHotfix()
	{
		return ENGINE_VERSION_HOTFIX;
	}

	const TCHAR* GetEngineVersionString()
	{
		return TEXT(ENGINE_VERSION_STRING);
	}

	int GetCurrentChangelist()
	{
		return CURRENT_CHANGELIST;
	}

	int GetCompatibleChangelist()
	{
		return COMPATIBLE_CHANGELIST;
	}

	const TCHAR* GetBranchName()
	{
		return TEXT(BRANCH_NAME);
	}
	
	const TCHAR* GetBuildDate()
	{
		return TEXT(__DATE__);
	}

	const TCHAR* GetBuildTime()
	{
		return TEXT(__TIME__);
	}

	const TCHAR* GetBuildVersion()
	{
		return TEXT(BUILD_VERSION);
	}

	bool IsPromotedBuild()
	{
		return ENGINE_IS_PROMOTED_BUILD;
	}
	
	bool IsWithDebugInfo()
	{
		return UE_WITH_DEBUG_INFO;
	}
	
	const TCHAR* GetBuildURL()
	{
		return TEXT(BUILD_SOURCE_URL);
	}

	const TCHAR* GetBuildUser()
	{
		return TEXT(BUILD_USER);
	}

	const TCHAR* GetBuildUserDomain()
	{
		return TEXT(BUILD_USERDOMAINNAME);
	}

	const TCHAR* GetBuildMachine()
	{
		return TEXT(BUILD_MACHINENAME);
	}

	const TCHAR* GetLiveCodingEngineDir()
	{
		#ifdef UE_LIVE_CODING_ENGINE_DIR
		return TEXT(UE_LIVE_CODING_ENGINE_DIR);
		#else
		return nullptr;
		#endif
	}

	const TCHAR* GetLiveCodingProject()
	{
		#ifdef UE_LIVE_CODING_PROJECT
		return TEXT(UE_LIVE_CODING_PROJECT);
		#else
		return nullptr;
		#endif
	}

	uint64 GetPersistentAllocatorReserveSize()
	{
		return UE_PERSISTENT_ALLOCATOR_RESERVE_SIZE;
	}

	const char* GetVfsPaths()
	{
		return UE_VFS_PATHS;
	}

	const TCHAR* GetVfsPathsWide()
	{
		return TEXT(UE_VFS_PATHS);
	}

	bool GetVariable(const TCHAR* InName, TCHAR* OutValue, int OutValueCapacity)
	{
		#ifdef UE_BUILDSETTINGS_VARIABLES
		const TCHAR* Name = TEXT(UE_BUILDSETTINGS_VARIABLES);
		while (true)
		{
			const TCHAR* NameEnd = FCString::Strchr(Name, ';');
			if (!NameEnd)
			{
				return false;
			}

			const TCHAR* Value = NameEnd + 1;
			const TCHAR* ValueEnd = FCString::Strchr(Value, ';');

			FStringView NameView(Name, int32(NameEnd - Name));
			if (NameView.Equals(InName))
			{
				if (OutValue)
				{
					FStringView ValueView(Value, int32(ValueEnd - Value));

					checkf(OutValueCapacity > ValueView.Len(), TEXT("Capacity of string sent in to BuildSettings::GetVariable is too short"));

					int32 CopyCount = ValueView.CopyString(OutValue, OutValueCapacity, 0);
					OutValue[CopyCount] = 0;
				}
				return true;
			}

			Name = ValueEnd + 1;

		}
		#endif

		return false;
	}
}
