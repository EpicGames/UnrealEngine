// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

namespace BuildSettings
{
	/**
	 * Determines if the changelist version numbers are from a licensee Perforce server. For the same major/minor/patch release of the engine, licensee changelists are always 
	 * considered newer than Epic changelists for engine versions. This follows the assumption that content is developed by Epic leading up to a release, and which 
	 * point we lock compatibility, and any subsequent licensee modifications to the engine will have a superset of its functionality even if the changelist 
	 * numbers are lower.
	 *
	 * @return Whether the changelist version numbers are from a licensee Perforce server
	 */
	BUILDSETTINGS_API bool IsLicenseeVersion();

	/**
	* The major engine version.
	* 
	* @return The major part of the engine version
	*/
	BUILDSETTINGS_API int GetEngineVersionMajor();

	/**
	* The minor engine version.
	*
	* @return The minor part of the engine version
	*/
	BUILDSETTINGS_API int GetEngineVersionMinor();

	/**
	* The hotfix engine version.
	*
	* @return The hotfix patch part of the engine version
	*/
	BUILDSETTINGS_API int GetEngineVersionHotfix();

	/**
	* The engine version.
	* 
	* @return The engine version as a string in the format MAJOR.MINOR.HOTFIX-BUILD_VERSION
	*/
	BUILDSETTINGS_API const TCHAR* GetEngineVersionString();

	/**
	 * The Perforce changelist being compiled. Use this value advisedly; it does not take into account out-of-order commits to engine release branches over 
	 * development branches, licensee versions, or whether the engine version has been locked to maintain compatibility with a previous engine release. Prefer
	 * BUILD_VERSION where a unique, product-specific identifier is required, or FEngineVersion::CompatibleWith() where relational comparisons between two 
	 * versions is required.
	 *
	 * @return The changelist number being compiled
	 */
	BUILDSETTINGS_API int GetCurrentChangelist();

	/**
	 * The compatible changelist version of the engine. This number identifies a particular API revision, and is used to determine module and package backwards 
	 * compatibility. Hotfixes should retain the compatible version of the original release. This define is parsed by the build tools, and should be a number or 
	 * BUILT_FROM_CHANGELIST, defined in this particular order for each alternative.
	 *
	 * @return The changelist number that this engine version is compatible with
	 */
	BUILDSETTINGS_API int GetCompatibleChangelist();

	/**
	 * The branch that this program is being built from.
	 *
	 * @return Name of the current branch, with slashes escaped as '+' characters.
	 */
	BUILDSETTINGS_API const TCHAR* GetBranchName();

	/**
	 * The date timestamp of this build. Derived from the compiler's __DATE__ macro, so only updated when the BuildSettings module is rebuilt (ie. whenever CL changes, etc...)
	 *
	 * @return String representing the build timestamp
	 */
	BUILDSETTINGS_API const TCHAR* GetBuildDate();

	/**
	 * The time timestamp of this build. Derived from the compiler's __TIME__ macro, so only updated when the BuildSettings module is rebuilt (ie. whenever CL changes, etc...)
	 *
	 * @return String representing the build timestamp
	 */
	BUILDSETTINGS_API const TCHAR* GetBuildTime();

	/**
	 * Retrieves the user-defined build version for this application.
	 *
	 * @return The current build version
	 */
	BUILDSETTINGS_API const TCHAR* GetBuildVersion();

	/**
	 * Identifies whether this build is a promoted build; a formal build of the engine from a clean source sync.
	 *
	 * @return True if this is a promoted build of the engine.
	 */
	BUILDSETTINGS_API bool IsPromotedBuild();

	/**
	 * Identifies whether this build was compiled with or without debug info. (e.g. pdb files on Microsoft platforms)
	 */
	BUILDSETTINGS_API bool IsWithDebugInfo();

	/**
	 * Returns a URL where the job which created these binaries on an automation system (e.g. Horde) can be found, or an empty string.
	 */
	BUILDSETTINGS_API const TCHAR* GetBuildURL();

	/**
	 * Returns the name of the user which created these binaries
	 */
	BUILDSETTINGS_API const TCHAR* GetBuildUser();

	/**
	 * Returns the domain name of the user which created these binaries
	 */
	BUILDSETTINGS_API const TCHAR* GetBuildUserDomain();

	/**
	 * Returns the name of the machine which created these binaries
	 */
	BUILDSETTINGS_API const TCHAR* GetBuildMachine();

	/**
	 * Returns the live coding engine dir. Only set in monolithic builds (live coding engine dir is calculated differently in modular builds)
	 */
	BUILDSETTINGS_API const TCHAR* GetLiveCodingEngineDir();

	/**
	 * Returns the live coding project. Only set in monolithic builds
	 */
	BUILDSETTINGS_API const TCHAR* GetLiveCodingProject();

	/**
	 * Persistent linear allocator reserve size. Stored here to prevent rebuilds of Core when switching between projects
	 */
	BUILDSETTINGS_API uint64 GetPersistentAllocatorReserveSize();

	/**
	 * Semicolon separated string containing virtual;local;virtual;local; paths. Ends with semicolon
	 */
	BUILDSETTINGS_API const char* GetVfsPaths();

	/**
	 * Semicolon separated string containing virtual;local;virtual;local; paths. Ends with semicolon
	 */
	BUILDSETTINGS_API const TCHAR* GetVfsPathsWide();

	/**
	 * Returns true if variable exists. Writes value into OutValue if not null
	 */
	BUILDSETTINGS_API bool GetVariable(const TCHAR* InName, TCHAR* OutValue, int OutValueCapacity);
}
